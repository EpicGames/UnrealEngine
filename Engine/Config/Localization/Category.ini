[CommonSettings]
ManifestDependencies=Content/Localization/Editor/Editor.manifest
SourcePath=Content/Localization/Category
DestinationPath=Content/Localization/Category
ManifestName=Category.manifest
ArchiveName=Category.archive
PortableObjectName=Category.po
NativeCulture=en
CulturesToGenerate=en
CulturesToGenerate=de
CulturesToGenerate=pl
CulturesToGenerate=es
CulturesToGenerate=es-419
CulturesToGenerate=fr
CulturesToGenerate=ja
CulturesToGenerate=ar
CulturesToGenerate=ko
CulturesToGenerate=pt-BR
CulturesToGenerate=it
CulturesToGenerate=ru
CulturesToGenerate=tr
CulturesToGenerate=zh-Hans

;Gather text from metadata
[GatherTextStep0]
CommandletClass=GatherTextFromMetaData
ModulesToPreload=OutputLog
ModulesToPreload=StructViewer
ModulesToPreload=MeshPaintMode
ModulesToPreload=LandscapeEditor
IncludePathFilters=Source/Editor/*
IncludePathFilters=Source/Runtime/*
IncludePathFilters=Source/Developer/*
IncludePathFilters=Plugins/*
ExcludePathFilters=*/NoRedist/*
ExcludePathFilters=*/NotForLicensees/*
ExcludePathFilters=*/LimitedAccess/*
InputKeys=Category
OutputNamespaces=UObjectCategory
OutputKeys="{MetaDataValue}"
ShouldGatherFromEditorOnlyData=true

;Write Manifest
[GatherTextStep1]
CommandletClass=GenerateGatherManifest

;Write Archives
[GatherTextStep2]
CommandletClass=GenerateGatherArchive
bPurgeOldEmptyEntries=true

;Import localized PO files
[GatherTextStep3]
CommandletClass=InternationalizationExport
bImportLoc=true

;Write Localized Text Resource
[GatherTextStep4]
CommandletClass=GenerateTextLocalizationResource
ResourceName=Category.locres

;Export to PO files
[GatherTextStep5]
CommandletClass=InternationalizationExport
bExportLoc=true

;Write Text Localization Report
[GatherTextStep6]
CommandletClass=GenerateTextLocalizationReport
DestinationPath=\\epicgames.net\root\UE3\Localization\WordCounts
bWordCountReport=true
WordCountReportName=Category.csv
bConflictReport=true
; Different file extensions will output the report in different formats. Currently supported extensions are .txt and .csv
ConflictReportName=CategoryConflicts.csv