[CommonSettings]
ManifestDependencies=Content/Localization/Engine/Engine.manifest
SourcePath=Content/Localization/Editor
DestinationPath=Content/Localization/Editor
ManifestName=Editor.manifest
ArchiveName=Editor.archive
PortableObjectName=Editor.po
NativeCulture=en
CulturesToGenerate=en
CulturesToGenerate=de
CulturesToGenerate=pl
CulturesToGenerate=es
CulturesToGenerate=es-419
CulturesToGenerate=fr
CulturesToGenerate=ja
CulturesToGenerate=ar
CulturesToGenerate=ko
CulturesToGenerate=pt-BR
CulturesToGenerate=it
CulturesToGenerate=ru
CulturesToGenerate=tr
CulturesToGenerate=zh-Hans

;Gather text from source code
[GatherTextStep0]
CommandletClass=GatherTextFromSource
SearchDirectoryPaths=Source/Editor/
SearchDirectoryPaths=Plugins/
ExcludePathFilters=Plugins/Online/*
ExcludePathFilters=Plugins/Runtime/HTTPChunkInstaller/*
ExcludePathFilters=*/NoRedist/*
ExcludePathFilters=*/NotForLicensees/*
ExcludePathFilters=*/LimitedAccess/*
ExcludePathFilters=*/ThirdParty/*
FileNameFilters=*.cpp
FileNameFilters=*.h
FileNameFilters=*.c
FileNameFilters=*.inl
FileNameFilters=*.mm
ShouldGatherFromEditorOnlyData=true

;Gather text from assets
[GatherTextStep1]
CommandletClass=GatherTextFromAssets
IncludePathFilters=Content/Editor/*
IncludePathFilters=Content/Editor*
IncludePathFilters=Plugins/PCG/*
PackageFileNameFilters=*.umap
PackageFileNameFilters=*.uasset
bFixBroken=false
ShouldGatherFromEditorOnlyData=true

;Gather text from metadata
[GatherTextStep2]
CommandletClass=GatherTextFromMetaData
ModulesToPreload=StructViewer
ModulesToPreload=MeshPaintMode
ModulesToPreload=LandscapeEditor
IncludePathFilters=Source/Editor/*
IncludePathFilters=Source/Runtime/*
IncludePathFilters=Source/Developer/*
IncludePathFilters=Plugins/*
ExcludePathFilters=Plugins/MetaHuman/*
ExcludePathFilters=*/NoRedist/*
ExcludePathFilters=*/NotForLicensees/*
ExcludePathFilters=*/LimitedAccess/*
ExcludePathFilters=Plugins/*/MetaHuman*
FieldTypesToInclude=Property
; general settings
FieldOwnerTypesToInclude=DeveloperSettings
FieldOwnerTypesToInclude=*Settings
; explict settings missed by the above filters
FieldOwnerTypesToInclude=AISystem
FieldOwnerTypesToInclude=CrowdManager
FieldOwnerTypesToInclude=Engine
FieldOwnerTypesToInclude=GameplayDebuggerConfig
FieldOwnerTypesToInclude=InternationalizationSettingsModel
FieldOwnerTypesToInclude=NavigationSystemV1
FieldOwnerTypesToInclude=PersonaOptions
FieldOwnerTypesToInclude=RecastNavMesh
; modes panel
FieldOwnerTypesToInclude=*Landscape*
InputKeys=DisplayName
OutputNamespaces=UObjectDisplayNames
OutputKeys="{FieldPath}"
InputKeys=Category
OutputNamespaces=UObjectCategory
OutputKeys="{MetaDataValue}"
ShouldGatherFromEditorOnlyData=true

;Write Manifest
[GatherTextStep3]
CommandletClass=GenerateGatherManifest

;Write Archives
[GatherTextStep4]
CommandletClass=GenerateGatherArchive
bPurgeOldEmptyEntries=true

;Import localized PO files
[GatherTextStep5]
CommandletClass=InternationalizationExport
bImportLoc=true

;Write Localized Text Resource
[GatherTextStep6]
CommandletClass=GenerateTextLocalizationResource
ResourceName=Editor.locres

;Export to PO files
[GatherTextStep7]
CommandletClass=InternationalizationExport
bExportLoc=true

;Write Text Localization Report
[GatherTextStep8]
CommandletClass=GenerateTextLocalizationReport
DestinationPath=\\epicgames.net\root\UE3\Localization\WordCounts
bWordCountReport=true
WordCountReportName=Editor.csv
bConflictReport=true
; Different file extensions will output the report in different formats. Currently supported extensions are .txt and .csv
ConflictReportName=EditorConflicts.csv