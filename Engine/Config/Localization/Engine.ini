[CommonSettings]
SourcePath=Content/Localization/Engine
DestinationPath=Content/Localization/Engine
ManifestName=Engine.manifest
ArchiveName=Engine.archive
PortableObjectName=Engine.po
NativeCulture=en
CulturesToGenerate=en
CulturesToGenerate=de
CulturesToGenerate=pl
CulturesToGenerate=es
CulturesToGenerate=es-419
CulturesToGenerate=fr
CulturesToGenerate=ja
CulturesToGenerate=ar
CulturesToGenerate=ko
CulturesToGenerate=pt-BR
CulturesToGenerate=it
CulturesToGenerate=ru
CulturesToGenerate=tr
CulturesToGenerate=zh-Hans

;Gather text from source code
[GatherTextStep0]
CommandletClass=GatherTextFromSource
SearchDirectoryPaths=Source/Runtime/
SearchDirectoryPaths=Source/Developer/
SearchDirectoryPaths=Plugins/
SearchDirectoryPaths=Config/
SearchDirectoryPaths=Source/Programs/CrashReportClient/
ExcludePathFilters=Config/Localization/*
ExcludePathFilters=Source/Runtime/Online/BuildPatchServices/*
ExcludePathFilters=Plugins/*Editor*/*
ExcludePathFilters=Plugins/Online/*
ExcludePathFilters=Plugins/Runtime/HTTPChunkInstaller/*
ExcludePathFilters=*/NoRedist/*
ExcludePathFilters=*/NotForLicensees/*
ExcludePathFilters=*/LimitedAccess/*
ExcludePathFilters=*/ThirdParty/*
FileNameFilters=*.cpp
FileNameFilters=*.h
FileNameFilters=*.c
FileNameFilters=*.inl
FileNameFilters=*.mm
FileNameFilters=*.ini
ShouldGatherFromEditorOnlyData=true

;Gather text from assets
[GatherTextStep1]
CommandletClass=GatherTextFromAssets
IncludePathFilters=Content/*
ExcludePathFilters=Content/Editor/*
ExcludePathFilters=Content/Developers/*
ExcludePathFilters=Content/Localization/*
ExcludePathFilters=Content/TestPackages/*
ExcludePathFilters=Content/QA_Assets/*
ExcludePathFilters=Content/EngineSounds/*
ExcludePathFilters=Content/Maps/Automation/*
ExcludePathFilters=Content/Tutorial/*
PackageFileNameFilters=*.umap
PackageFileNameFilters=*.uasset
bFixBroken=false
ShouldGatherFromEditorOnlyData=true

;Write Manifest
[GatherTextStep2]
CommandletClass=GenerateGatherManifest

;Write Archives
[GatherTextStep3]
CommandletClass=GenerateGatherArchive
bPurgeOldEmptyEntries=true

;Import localized PO files
[GatherTextStep4]
CommandletClass=InternationalizationExport
bImportLoc=true

;Write Localized Text Resource
[GatherTextStep5]
CommandletClass=GenerateTextLocalizationResource
ResourceName=Engine.locres

;Export  PO files
[GatherTextStep6]
CommandletClass=InternationalizationExport
bExportLoc=true

;Write Text Localization Report
[GatherTextStep7]
CommandletClass=GenerateTextLocalizationReport
DestinationPath=\\epicgames.net\root\UE3\Localization\WordCounts
bWordCountReport=true
WordCountReportName=Engine.csv
bConflictReport=true
; Different file extensions will output the report in different formats. Currently supported extensions are .txt and .csv
ConflictReportName=EngineConflicts.csv