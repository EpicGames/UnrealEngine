[CommonSettings]
bImportLoc=true
NativeCulture=en
CulturesToGenerate=en
CulturesToGenerate=es
CulturesToGenerate=ja
CulturesToGenerate=ko
CulturesToGenerate=pt-BR
CulturesToGenerate=zh-Hans

[GatherTextStep0]
CommandletClass=InternationalizationExport
SourcePath=Content/Localization/Engine
DestinationPath=Content/Localization/Engine
ManifestName=Engine.manifest
ArchiveName=Engine.archive
PortableObjectName=Engine.po

[GatherTextStep1]
CommandletClass=InternationalizationExport
SourcePath=Content/Localization/Editor
DestinationPath=Content/Localization/Editor
ManifestName=Editor.manifest
ArchiveName=Editor.archive
PortableObjectName=Editor.po

[GatherTextStep2]
CommandletClass=InternationalizationExport
SourcePath=Content/Localization/ToolTips
DestinationPath=Content/Localization/ToolTips
ManifestName=ToolTips.manifest
ArchiveName=ToolTips.archive
PortableObjectName=ToolTips.po

[GatherTextStep3]
CommandletClass=InternationalizationExport
SourcePath=Content/Localization/PropertyNames
DestinationPath=Content/Localization/PropertyNames
ManifestName=PropertyNames.manifest
ArchiveName=PropertyNames.archive
PortableObjectName=PropertyNames.po

[GatherTextStep4]
CommandletClass=InternationalizationExport
SourcePath=Content/Localization/EditorTutorials
DestinationPath=Content/Localization/EditorTutorials
ManifestName=EditorTutorials.manifest
ArchiveName=EditorTutorials.archive
PortableObjectName=EditorTutorials.po