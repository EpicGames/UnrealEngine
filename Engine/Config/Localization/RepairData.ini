[CommonSettings]
CulturesToGenerate=en
CulturesToGenerate=es
CulturesToGenerate=ja
CulturesToGenerate=ko
CulturesToGenerate=pt-BR
CulturesToGenerate=zh-Hans

;Repair localization data for Engine
[GatherTextStep0]
CommandletClass=RepairLocalizationData
DestinationPath=Content/Localization/Engine
ManifestName=Engine.manifest
ArchiveName=Engine.archive

;Write Localized Text Resource for Engine
[GatherTextStep1]
CommandletClass=GenerateTextLocalizationResource
SourcePath=Content/Localization/Engine
DestinationPath=Content/Localization/Engine
ManifestName=Engine.manifest
ResourceName=Engine.locres

;Repair localization data for Editor
[GatherTextStep2]
CommandletClass=RepairLocalizationData
DestinationPath=Content/Localization/Editor
ManifestName=Editor.manifest
ArchiveName=Editor.archive

;Write Localized Text Resource for Editor
[GatherTextStep3]
CommandletClass=GenerateTextLocalizationResource
SourcePath=Content/Localization/Editor
DestinationPath=Content/Localization/Editor
ManifestName=Editor.manifest
ResourceName=Editor.locres

;Repair localization data for Editor Tool Tips
[GatherTextStep4]
CommandletClass=RepairLocalizationData
DestinationPath=Content/Localization/ToolTips
ManifestName=ToolTips.manifest
ArchiveName=ToolTips.archive

;Write Localized Text Resource for Editor Tool Tips
[GatherTextStep5]
CommandletClass=GenerateTextLocalizationResource
SourcePath=Content/Localization/ToolTips
DestinationPath=Content/Localization/ToolTips
ManifestName=ToolTips.manifest
ResourceName=ToolTips.locres

;Repair localization data for Editor Property Names
[GatherTextStep6]
CommandletClass=RepairLocalizationData
DestinationPath=Content/Localization/PropertyNames
ManifestName=PropertyNames.manifest
ArchiveName=PropertyNames.archive

;Write Localized Text Resource for Editor Property Names
[GatherTextStep7]
CommandletClass=GenerateTextLocalizationResource
SourcePath=Content/Localization/PropertyNames
DestinationPath=Content/Localization/PropertyNames
ManifestName=PropertyNames.manifest
ResourceName=PropertyNames.locres

;Repair localization data for Editor Tutorials
[GatherTextStep8]
CommandletClass=RepairLocalizationData
DestinationPath=Content/Localization/PropertyNames
ManifestName=PropertyNames.manifest
ArchiveName=PropertyNames.archive

;Write Localized Text Resource for Editor Tutorials
[GatherTextStep9]
CommandletClass=GenerateTextLocalizationResource
SourcePath=Content/Localization/EditorTutorials
DestinationPath=Content/Localization/EditorTutorials
ManifestName=EditorTutorials.manifest
ResourceName=EditorTutorials.locres