[CommonSettings]
bWordCountReport=true
DestinationPath=\\epicgames.net\root\UE3\Localization\WordCounts
CulturesToGenerate=de
CulturesToGenerate=pl
CulturesToGenerate=es
CulturesToGenerate=es-419
CulturesToGenerate=fr
CulturesToGenerate=ja
CulturesToGenerate=ar
CulturesToGenerate=ko
CulturesToGenerate=pt-BR
CulturesToGenerate=it
CulturesToGenerate=ru
CulturesToGenerate=tr
CulturesToGenerate=zh-Hans

;Write Text Localization Report for Engine
[GatherTextStep0]
CommandletClass=GenerateTextLocalizationReport
SourcePath=Content/Localization/Engine
ManifestName=Engine.manifest
WordCountReportName=Engine.csv

;Write Text Localization Report for Editor
[GatherTextStep1]
CommandletClass=GenerateTextLocalizationReport
SourcePath=Content/Localization/Editor
ManifestName=Editor.manifest
WordCountReportName=Editor.csv

;Write Text Localization Report for Editor Tool Tips
[GatherTextStep2]
CommandletClass=GenerateTextLocalizationReport
SourcePath=Content/Localization/ToolTips
ManifestName=ToolTips.manifest
WordCountReportName=ToolTips.csv

;Write Text Localization Report for Editor Property Names
[GatherTextStep3]
CommandletClass=GenerateTextLocalizationReport
SourcePath=Content/Localization/PropertyNames
ManifestName=PropertyNames.manifest
WordCountReportName=PropertyNames.csv

;Write Text Localization Report for Editor Tutorials
[GatherTextStep4]
CommandletClass=GenerateTextLocalizationReport
SourcePath=Content/Localization/EditorTutorials
ManifestName=EditorTutorials.manifest
WordCountReportName=EditorTutorials.csv

;Write Text Localization Report for Editor Categories
[GatherTextStep5]
CommandletClass=GenerateTextLocalizationReport
SourcePath=Content/Localization/Category
ManifestName=Category.manifest
WordCountReportName=Category.csv

;Write Text Localization Report for Editor Keywords
[GatherTextStep6]
CommandletClass=GenerateTextLocalizationReport
SourcePath=Content/Localization/Keywords
ManifestName=Keywords.manifest
WordCountReportName=Keywords.csv